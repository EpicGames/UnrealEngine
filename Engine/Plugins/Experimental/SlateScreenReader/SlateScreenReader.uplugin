{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "SlateScreenReader",
	"Description" : "A screen reader that provides vision accessibility services for Slate.",
	"Category" : "Accessibility",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsExperimental" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "SlateScreenReader",
			"Type" : "Runtime",
			"LoadingPhase" : "PostEngineInit",
			"PlatformAllowList" :
			[
				"Win64",
				"Mac",
				"Linux",
				"LinuxArm64"
			],
			"TargetDenyList" :
			[
				"Server"
			]
		}
	],
	"Plugins" :
	[
		{
			"Name" : "ScreenReader",
			"Enabled": true
		}
	]
}
