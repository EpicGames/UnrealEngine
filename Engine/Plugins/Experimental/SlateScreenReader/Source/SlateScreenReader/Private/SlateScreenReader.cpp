// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateScreenReader.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/ScreenReaderReply.h"
#include "GenericPlatform/ScreenReaderUser.h"
#include "SlateScreenReaderLog.h"


FSlateScreenReader::FSlateScreenReader(const TSharedRef<GenericApplication>& InPlatformApplication)
	: FScreenReaderBase(InPlatformApplication)
{
	// if we are creating and using FSlateScreenReader, we assume that other accessibility options (such as platform accessibility) are going to be overridden
	// We unregister all previously registered users and force registration through the screen reader framework
	// @TODOAccessibility: It would be better to swap the existing FGenericAccessibleUsers for their FScreenReaderUser counterparts, but we will deal with that later
	InPlatformApplication->GetAccessibleMessageHandler()->GetAccessibleUserRegistry().UnregisterAllUsers();
}

FSlateScreenReader::~FSlateScreenReader()
{

}

void FSlateScreenReader::OnAccessibleEventRaised(const FAccessibleEventArgs& Args)
{
	// This should only be triggered by the accessible message handler which initiates from the Slate thread.
	check(IsInGameThread());
if (IsActive())
{
	switch (Args.Event)
	{
		case EAccessibleEvent::FocusChange:
		{
			// if the widget is gaining focus
			if (Args.NewValue.GetValue<bool>())
			{
				TSharedPtr<FScreenReaderUser> User = GetUser(Args.UserIndex);
				if (User)
				{
					User->RequestSpeakWidget(Args.Widget);
				}
			}
			break;
		}
	}
}
}

void FSlateScreenReader::OnActivate()
{
	UE_LOGF(LogSlateScreenReader, Verbose, "On activation of Slate screen reader.");
	// @TODOAccessibility: Provide feedback in the form of a tone or something
}

void FSlateScreenReader::OnDeactivate()
{
	UE_LOGF(LogSlateScreenReader, Verbose, "Deactivation of Slate screen reader.");
	// @TODOAccessibility: Provide feedback in the form of a tone or something
} 

