// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/ScreenReaderBase.h"

class iAccessibleWidget;
struct FAccessibleEventArgs;


/**
 * A basic screen reader that can work for desktop and consoles that use Slate.
 * All feedback to users are done through speech.
 */
class FSlateScreenReader
	: public FScreenReaderBase
{
public:
	FSlateScreenReader() = delete;
	FSlateScreenReader(const FSlateScreenReader& Other) = delete;
	explicit FSlateScreenReader(const TSharedRef<GenericApplication>& InPlatformApplication);
	virtual ~FSlateScreenReader();
	
	FSlateScreenReader& operator=(const FSlateScreenReader& Other) = delete;
	
protected:
	//~ Begin FScreenReaderBase interface
	virtual void OnActivate() override;
	virtual void OnDeactivate() override;
	virtual void OnAccessibleEventRaised(const FAccessibleEventArgs& Args) override;
	//~ End FScreenReaderBase interface
};

