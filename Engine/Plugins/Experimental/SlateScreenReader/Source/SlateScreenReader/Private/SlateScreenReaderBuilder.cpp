// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateScreenReaderBuilder.h"
#include "SlateScreenReader.h"

TSharedRef<FScreenReaderBase> FSlateScreenReaderBuilder::Create(const IScreenReaderBuilder::FArgs& InArgs)
{
	TSharedRef<FScreenReaderBase> ScreenReader = MakeShared<FSlateScreenReader>(InArgs.PlatformApplication);
	// @TODOAccessibility: Set custom screen reader application message handlers for different input handling 
	return ScreenReader;
}
