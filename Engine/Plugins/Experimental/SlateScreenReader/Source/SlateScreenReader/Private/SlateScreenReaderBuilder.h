// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/IScreenReaderBuilder.h"

class FSlateScreenReaderBuilder : public IScreenReaderBuilder
{
public:
	FSlateScreenReaderBuilder() {}
	virtual ~FSlateScreenReaderBuilder() {}

	virtual TSharedRef<FScreenReaderBase> Create(const IScreenReaderBuilder::FArgs& InArgs) override;

};
