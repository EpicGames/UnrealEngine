// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateScreenReaderEngineSubsystem.h"
#include "Announcement/ScreenReaderAnnouncement.h"
#include "GenericPlatform/ScreenReaderReply.h"
#include "GenericPlatform/ScreenReaderBase.h"
#include "GenericPlatform/ScreenReaderUser.h"
#include "SlateScreenReaderLog.h"
#include "GenericPlatform/IScreenReaderBuilder.h"
#include "SlateScreenReaderModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/Engine.h"


USlateScreenReaderEngineSubsystem::USlateScreenReaderEngineSubsystem()
{

}

USlateScreenReaderEngineSubsystem::~USlateScreenReaderEngineSubsystem()
{
	ensureMsgf(!ScreenReader, TEXT("Screen reader should already be destroyed and nulled at this point."));
}

USlateScreenReaderEngineSubsystem& USlateScreenReaderEngineSubsystem::Get()
{
	return *GEngine->GetEngineSubsystem<USlateScreenReaderEngineSubsystem>();
}

void USlateScreenReaderEngineSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	// By the time this is called, Slate is all set up
	// I.e Platform applications already have subscribed to the delegate in the accessible
	// message handler. So any platform stuff e.g OSX trees and cache are all set up.  That needs to get removed somehow when we subscribe to accessible events here
	FSlateApplication& SlateApplication = FSlateApplication::Get();
	SlateApplication.GetPlatformApplication()->GetAccessibleMessageHandler()->UnbindAccessibleEventDelegate();
	// @TODOAccessibility: We've unbound from the event, but we need to clean up the OS resources that are allocated for accessibility as well. 

	// @TODOAccessibility: Consider lazy initialization 
	IScreenReaderBuilder::FArgs Args(SlateApplication.GetPlatformApplication().ToSharedRef());
	// @TODOAccessibility: Allow a means of using custom screen reader builder 
	ScreenReader = ISlateScreenReaderModule::Get().GetDefaultScreenReaderBuilder()->Create(Args);
	// Slate could get shutdown prior to engine subsystems 
	SlateApplication.OnPreShutdown().AddSP(ScreenReader.ToSharedRef(), &FScreenReaderBase::Deactivate);
}

void USlateScreenReaderEngineSubsystem::Deinitialize()
{
	// should still be valid
	check(ScreenReader);
	// Engine subsystems are desroyed before FSlateApplication::Shutdown is called
	// We clean up here 
	if (ScreenReader->IsActive())
	{
		ScreenReader->Deactivate();
	}
	ScreenReader.Reset();
	Super::Deinitialize();
}

bool USlateScreenReaderEngineSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	// On certain builds, Slate is not initialize, we don't need the screen reader in that case 
	return FSlateApplication::IsInitialized();
}

void USlateScreenReaderEngineSubsystem::ActivateScreenReader()
{
	checkf(ScreenReader, TEXT("Screen reader is invalid during activation. A screen reader should have been created on subsystem initialization and be valid for the duration of the subsystem."));
	if (!ScreenReader->IsActive())
	{
		ScreenReader->Activate();
	}
}

void USlateScreenReaderEngineSubsystem::DeactivateScreenReader()
{
	checkf(ScreenReader, TEXT("Invalid screen reader on deactivation. A valid screen reader must always be available for deactivation."));
	if (ScreenReader->IsActive())
	{
		ScreenReader->Deactivate();
	}
}

bool USlateScreenReaderEngineSubsystem::IsScreenReaderActive() const
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	return ScreenReader->IsActive();
}

FScreenReaderReply USlateScreenReaderEngineSubsystem::ActivateUser(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId)
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	TSharedPtr<FScreenReaderUser> User = ScreenReader->GetUser(InUserId);
	if (User)
	{
		User->Activate();
		return FScreenReaderReply::Handled();
	}
	UE_LOGF(LogSlateScreenReader, Warning, "No screen reader user with %d Id exists. Did you forget to register the user Id?", InUserId);
	return FScreenReaderReply::Unhandled();
}

FScreenReaderReply USlateScreenReaderEngineSubsystem::DeactivateUser(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId)
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	TSharedPtr<FScreenReaderUser> User = ScreenReader->GetUser(InUserId);
	if (User)
	{
		// FScreenReaderUser already checks if it is active/inactive 
		User->Deactivate();
		return FScreenReaderReply::Handled();
	}
	UE_LOGF(LogSlateScreenReader, Warning, "No screen reader user with %d Id exists. Did you forget to register the user Id?", InUserId);
	return FScreenReaderReply::Unhandled();
}

FScreenReaderReply USlateScreenReaderEngineSubsystem::RegisterUser(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId)
{
	return ScreenReader->RegisterUser(InUserId)
		? FScreenReaderReply::Handled()
		: FScreenReaderReply::Unhandled();
}

FScreenReaderReply USlateScreenReaderEngineSubsystem::UnregisterUser(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId)
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	return ScreenReader->UnregisterUser(InUserId)
		? FScreenReaderReply::Handled()
		: FScreenReaderReply::Unhandled();
}

bool USlateScreenReaderEngineSubsystem::IsUserRegistered(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId) const
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	return ScreenReader->IsUserRegistered(InUserId);
}

FScreenReaderReply USlateScreenReaderEngineSubsystem::RequestSpeak(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId, UPARAM(DisplayName = "Announcement") FScreenReaderAnnouncement InAnnouncement)
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	if (ScreenReader->IsActive())
	{
		TSharedPtr<FScreenReaderUser> User = ScreenReader->GetUser(InUserId);
		if (User)
		{
			return User->RequestSpeak(MoveTemp(InAnnouncement));
		}
	}
	return FScreenReaderReply::Unhandled();
}

FScreenReaderReply USlateScreenReaderEngineSubsystem::RequestSpeakFocusedWidget(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId)
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	if (ScreenReader->IsActive())
	{
		TSharedPtr<FScreenReaderUser> User = ScreenReader->GetUser(InUserId);
		if (User)
		{
			if (TSharedPtr<IAccessibleWidget> UserFocusedWidget = User->GetFocusedAccessibleWidget())
			{
				return User->RequestSpeakWidget(UserFocusedWidget.ToSharedRef());
			}
		}
	}
	return FScreenReaderReply::Unhandled();
}

FScreenReaderReply USlateScreenReaderEngineSubsystem::StopSpeaking(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId)
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	if (IsScreenReaderActive())
	{
		TSharedPtr<FScreenReaderUser> User = ScreenReader->GetUser(InUserId);
		if (User)
		{
			User->StopSpeaking();
			return FScreenReaderReply::Handled();
		}
	}
	return FScreenReaderReply::Unhandled();
}

bool USlateScreenReaderEngineSubsystem::IsSpeaking(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId) const
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	if (ScreenReader->IsActive())
	{
		TSharedPtr<FScreenReaderUser> User = ScreenReader->GetUser(InUserId);
		if (User)
		{
			return User->IsSpeaking();
		}
	}
	return false;
}

float USlateScreenReaderEngineSubsystem::GetSpeechVolume(UPARAM(DisplayName = "Screen Reader User Id") UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId) const
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	TSharedPtr<FScreenReaderUser> User = ScreenReader->GetUser(InUserId);
	if (User)
	{
		return User->GetSpeechVolume();
	}
	return 0.0f;
}

FScreenReaderReply USlateScreenReaderEngineSubsystem::SetSpeechVolume(UPARAM(DisplayName = "Screen Reader User Id") UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId, UPARAM(DisplayName = "Volume") float InVolume)
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	TSharedPtr<FScreenReaderUser> User = ScreenReader->GetUser(InUserId);
	if (User)
	{
		return User->SetSpeechVolume(InVolume);
	}
	return FScreenReaderReply::Unhandled();
}

float USlateScreenReaderEngineSubsystem::GetSpeechRate(UPARAM(DisplayName = "Screen Reader User Id") UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId) const
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	TSharedPtr<FScreenReaderUser> User = ScreenReader->GetUser(InUserId);
	if (User)
	{
		return User->GetSpeechRate();
	}
	return 0.0f;
}

FScreenReaderReply USlateScreenReaderEngineSubsystem::SetSpeechRate(UPARAM(DisplayName = "Screen Reader User Id") UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId, UPARAM(DisplayName = "Rate") float InRate)
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	TSharedPtr<FScreenReaderUser> User = ScreenReader->GetUser(InUserId);
	if (User)
	{
		return User->SetSpeechRate(InRate);
	}
	return FScreenReaderReply::Unhandled();
}

FScreenReaderReply USlateScreenReaderEngineSubsystem::MuteSpeech(UPARAM(DisplayName = "Screen Reader User Id") UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId)
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	if (IsScreenReaderActive())
	{
		TSharedPtr<FScreenReaderUser> User = ScreenReader->GetUser(InUserId);
		if (User)
		{
			return User->MuteSpeech();
		}
	}
	return FScreenReaderReply::Unhandled();
}

FScreenReaderReply USlateScreenReaderEngineSubsystem::UnmuteSpeech(UPARAM(DisplayName = "Screen Reader User Id") UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId)
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	if (IsScreenReaderActive())
	{
		TSharedPtr<FScreenReaderUser> User = ScreenReader->GetUser(InUserId);
		if (User)
		{
			return User->UnmuteSpeech();
		}
	}
	return FScreenReaderReply::Unhandled();
}

bool USlateScreenReaderEngineSubsystem::IsSpeechMuted(UPARAM(DisplayName = "Screen Reader User Id") UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId) const
{
	checkf(ScreenReader, TEXT("Screen reader should have been created during subsystem initialization and should be valid for the entire lifetime of the engine subsystem."));
	if (IsScreenReaderActive())
	{
		TSharedPtr<FScreenReaderUser> User = ScreenReader->GetUser(InUserId);
		if (User)
		{
			return User->IsSpeechMuted();
		}
	}
	return false;
}

TSharedRef<FScreenReaderBase> USlateScreenReaderEngineSubsystem::GetScreenReader() const
{
	return ScreenReader.ToSharedRef();
}
