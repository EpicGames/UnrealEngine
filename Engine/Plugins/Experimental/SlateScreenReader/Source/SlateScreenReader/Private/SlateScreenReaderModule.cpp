// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateScreenReaderModule.h"
#include "SlateScreenReaderLog.h"
#include "SlateScreenReaderBuilder.h"



class FSlateScreenReaderModule : public ISlateScreenReaderModule
{
public:
	FSlateScreenReaderModule()
		: DefaultBuilder(MakeShared<FSlateScreenReaderBuilder>())
	{

	}
	virtual ~FSlateScreenReaderModule() = default;
	// ISlateScreenReaderModule
	virtual TSharedRef<IScreenReaderBuilder> GetDefaultScreenReaderBuilder() const override
	{
		return DefaultBuilder;
	}

	virtual TSharedPtr<IScreenReaderBuilder> GetCustomScreenReaderBuilder() const override
	{
		return CustomBuilder;
}
	
	virtual void SetCustomScreenReaderBuilder(const TSharedRef<IScreenReaderBuilder>& InBuilder) override
	{
		CustomBuilder= InBuilder;
	}
	// ~

	// IModuleInterface
	virtual void StartupModule() override
	{
		
	}

	virtual void ShutdownModule() override
	{
		
	}
	// ~
private:
	TSharedRef<IScreenReaderBuilder> DefaultBuilder;
	TSharedPtr<IScreenReaderBuilder> CustomBuilder;
};

IMPLEMENT_MODULE(FSlateScreenReaderModule, SlateScreenReader);

DEFINE_LOG_CATEGORY(LogSlateScreenReader);