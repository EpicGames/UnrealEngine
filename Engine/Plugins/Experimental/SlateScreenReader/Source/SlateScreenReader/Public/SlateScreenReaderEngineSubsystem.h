// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Subsystems/EngineSubsystem.h"
#include "Announcement/ScreenReaderAnnouncement.h"
#include "GenericPlatform/ScreenReaderReply.h"
#include "SlateScreenReaderEngineSubsystem.generated.h"

class FScreenReaderBase;
/**
* The engine subsystem for the Slate screen reader.
* A screen reader is a framework that provides vision accessibility services for screen reader useres.
* A screen reader user is a single user of the screen reader framework and can be thought of as a user of a hardware device such as keyboard/mouse or a controller.
* Screen reader users must be registered with the screen reader framework for them to receive feedback from the accessibility services provided by the screen reader framework.
* This class should be the entryway for C++ programmers and BP useres alike to interact with the screen reader system.
* The subsystem must be activated before the screen reader services can be used.
* For C++ users, please retrieve the screen reader and interact with the screen reader users from there.
* Example:
* USlateScreenReaderEngineSubsystem ::Get().ActivateScreenReader();
* // Registers a screen reader user with Id 0. A screen reader user should correspond to a hardware input device such as a keyboard or controller like FSlateUser
* USlateScreenReaderEngineSubsystem ::Get().GetScreenReader()->RegisterUser(0);
* TSharedRef<FScreenReaderUser> User = USlateScreenReaderEngineSubsystem::Get().GetScreenReader()->GetUserChecked(0);
* // Screen reader users are inactive when they are first registered and need to be explicitly activated.
* User->Activate();
* static const FText HelloWorld = LOCTEXT("HelloWorld", "Hello World");
* // Requests "Hello World" to be spoken to the screen reader user
* User->RequestSpeak(FScreenReaderAnnouncement(HelloWorld.ToString(), FScreenReaderInfo::Important())); 
* @see FScreenReaderBase, FScreenReaderUser, FScreenReaderAnnouncement
*/
UCLASS()
class SLATESCREENREADER_API USlateScreenReaderEngineSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()
public:
	USlateScreenReaderEngineSubsystem();
	virtual ~USlateScreenReaderEngineSubsystem();

	/** Convenience method to retrieve the screen reader engine subsystem. */
	static USlateScreenReaderEngineSubsystem& Get();
	
	/** 
	* Activates the underlying screen reader. Use this to allow screen reader users to register with the screen reader 
	* and receive accessible feedback via text to speech and get access to other screen reader services.
	* A basic workflow with activation would be:
	* ActivateScreenReader() -> RegisterScreenReaderUser() -> ActivateScreenReaderUser()
	* @see RegisterScreenReaderUser(), ActivateScreenReaderUser()
	*/
	UFUNCTION(BlueprintCallable, Category="SlateScreenReader")
	void ActivateScreenReader();

	/**
	* Deactivates the underlying screen reader and prevents screen reader users from getting
	* any accessible feedback via text to speech or using any other screen reader services.
	* Note: When the screen reader is deactivated, none of the registered screen reader users will be unregistered or cleared. This allows you to 
	* deactivate the screen reader to prevent accessible services such as text to speech from triggering and then activate the screen reader again to continue those services.
	*/
	UFUNCTION(BlueprintCallable, Category="SlateScreenReader")
	void DeactivateScreenReader();

	/**
	* Returns true if the screen reader is currently active and accessibility services such as text to speech can be used by the screen reader users. 
	* Otherwise, it returns false.
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	bool IsScreenReaderActive() const;

	/**
	* Activates a screen reader user and fulfill requests for accessibility services such as text to speech that clients can make.
	* When screen reader users are first registered with a screen reader, they are deactivated by default. Users must explicitly activate the screen reader user.
	* If the passed in user Id does not correspond to a registered screen reader user, nothing will happen.
	* @param InUserId The user Id of the screen reader user to request activation 
	* @return FScreenReaderReply::Handled() if the screen reader user is successfully activated. Else FScreenReaderReply::Unhandled() is returned.
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	FScreenReaderReply ActivateUser(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId);

	/**
	* Deactivates the screen reader and disables all announcement and text to speech services
	* making them do nothing.
	* If the passed in user Id does not correspond to a registered screen reader user, nothing will happen.
	* @param InUserId The user Id of the screen reader user to request deactivation. The Id should correspond to the Slate user id of an input device. If unsure, use Id 0. 
	* @return FScreenReaderReply::Handled() if the screen reader user is successfully deactivated. Else FScreenReaderReply::Unhandled() is returned.
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	FScreenReaderReply DeactivateUser(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId);

	/**
	* Registers a provided user Id to the screen reader framework and allows the screen reader user to receive and respond to accessible events and accessible input.
	* Does nothing if the passed in Id is already registered.
	* Note: A successfully registered screen reader user is deactivated by default and will not respond to accessible events or accessible input.
	* You need to call ActivateUser() to allow the newly registered screen reader user to respond to the accessible events and accessible input.
	* @param InUserId The user Id of the screen reader user to register. The Id should correspond to a valid Slate user Id for an active hardware input device. If unsure, use Id 0.
	* @return FScreenReaderReply::Handled() if the screen reader user is successfully registered. Else FScreenReaderReply::Unhandled() is returned.
	* @see ActivateUser()
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	FScreenReaderReply RegisterUser(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId);

	/**
	* Unregisters a provided user Id from the screen reader framework and deactivates the user. The unregistered user will no longer receive or respond to accessible events and input.
	* Nothing will happen if the provided user Id has not been registered with the screen reader.
	* @param InUserId The user Id to unregister from the screen reader framework.
	* @return FScreenReaderReply::Handled() if the screen reader user is successfully unregistered. Else FScreenReaderReply::Unhandled() is returned.
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	FScreenReaderReply UnregisterUser(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId);
	/**
	* Returns true if the passed in screen reader user Id is already registered. Else false is returned.
	* @param InUserId The user Id to check for registration.
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	bool IsUserRegistered(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId) const;

	/**
	* Requests an announcement to be spoken to the screen reader user. This is the main mechanism to provide text to speech auditory feedback
	* to end users.
	*Calling this function does not guarantee that the announcement will be spoken via text to speech and be
	* heard by a user.
	* All announcements spoken via text to speech will be asynchronous and will not block the game thread.
	* If the screen reader user is active and no announcements are currently spoken, the announcement will be spoken immediately.
	* If another announcement is currently being spoken, the passed in announcement could be queued or interrupt the currently spoken announcement.
	* @param InUserId The user Id of the screen reader user the announcement is intended for 
	* @param InAnnouncement - The announcement requested to be spoken to the screen reader user
	* @return FScreenReaderReply::Handled() if the request was successfully processed. Else return FScreenReader::Unhandled()
	* @see FScreenReaderAnnouncement, FScreenReaderAnnouncementInfo
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	FScreenReaderReply RequestSpeak(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId, UPARAM(DisplayName = "Announcement") FScreenReaderAnnouncement InAnnouncement);

	/**
	* Requests the information about the accessibility widget a user is focused on to be read out.
	* If nothing is currently being focused on by the screen reader user, nothing will be read out.
	* The same guarantees about the announcement being spoken in RequestSpeak() apply for this function.
	* Nothing will happen if the passed in user Id is not already registered with the screen reader.
	* @param InUserId The user Id to request its accessible focus to be read out
	* @return FScreenReaderReply::Handled() if the user's focused widget's information is successfully spoken. Else FScreenReaderReply::Unhandled() is returned. 
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	FScreenReaderReply RequestSpeakFocusedWidget(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId);

	/**
	* Immediately stops speaking any currently spoken announcement for a particular screen reader user.
	* Does nothing if there is no announcement currently being spoken for the user or if the user Id is not registered with the screen reader.
	* @param InUserId The user Id of the screen reader user to request announcements to be stopped.
	* @return FScreenReader::Handled() if the request to stop speaking is successfully processed. Else FScreenReaderReply::Unhandled() is returned.
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	FScreenReaderReply StopSpeaking(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId);

	/**
	* Returns true if the screen reader is speaking text to a particular user.
	* Returns false if no no announcements are being spoken to the user or if the user Id is not registered.
	* @param InUserId The user Id of the screen reader user to check if any announcements are being spoken to.
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	bool IsSpeaking(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId) const;

	/**
	* Returns the volume text to speech will be speaking at for a screen reader user.  Value is between 0.0f and 1.0f.
	* If the provided user Id doesn't exist, 0.0f will be returned.
	* @param InUserId The user Id of the screen reader user to retrieve the text to speech volume for 
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	float GetSpeechVolume(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId) const;

	/**
	* Sets the volume text to speech will be speaking at for a screen reader user.  
	* If the provided user Id doesn't exist, nothing will happen.
	* @param InUserId The user Id of the screen reader user to set the text to speech volume for
	* @param InVolume The volume text to speech will be set at for the provided screen reader user. Value will be clamped between 0.0f and 1.0f.
	* @return FScreenReaderReply::Handled() is returned if the speech volume is successfully set for the screen reader user. Otherwise, FScreenReaderReply::Unhandled() is returned.
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	FScreenReaderReply SetSpeechVolume(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId, UPARAM(DisplayName = "Volume") float InVolume);

	/**
	* Returns the rate text to speech will be speaking at for a screen reader user.  Value is between 0.0f and 1.0f.
	* If the provided user Id doesn't exist, 0.0f will be returned.
	* @param InUserId The user Id of the screen reader user to retrieve the text to speech rate for
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	float GetSpeechRate(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId) const;

	/**
	* Sets the rate text to speech will be speaking at for a screen reader user.
	* If the provided user Id doesn't exist, nothing will happen.
	* @param InUserId The user Id of the screen reader user to set the text to speech rate for
	* @param InRate The rate text to speech will be set at for the provided screen reader user. Value will be clamped between 0.0f and 1.0f.
	* @return FScreenReaderReply::Handled() is returned if the speech rate is successfully set for the screen reader user. Otherwise, FScreenReaderReply::Unhandled() is returned.
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	FScreenReaderReply SetSpeechRate(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId, UPARAM(DisplayName = "Rate") float InRate);

	/**
	* Mutes the text to speech for a screen reader user.
	* If the provided user Id doesn't exist, nothing will happen.
	* @param InUserId The user Id of the screen reader user to mute text to speech for.
	* @return FScreenReaderReply::Handled() is returned if the screen reader user is successfully muted. Otherwise, FScreenReaderReply::Unhandled() is returned.
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	FScreenReaderReply MuteSpeech(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId);

	/**
	* Unmutes the text to speech for a screen reader user.
	* If the provided user Id doesn't exist, nothing will happen.
	* @param InUserId The user Id of the screen reader user to unmute text to speech for.
	* @return FScreenReaderReply::Handled() is returned if the screen reader user is successfully unmuted. Otherwise, FScreenReaderReply::Unhandled() is returned.
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	FScreenReaderReply UnmuteSpeech(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId);

	/**
	* Returns true if text to speech for a screen reader user is muted. Otherwise false is returned.
	* If the provided user Id doesn't exist, false will be returned.
	* @param InUserId The screen reader user Id to check if text to speech is muted.
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateScreenReader")
	bool IsSpeechMuted(UPARAM(DisplayName = "Screen Reader User Id") int32 InUserId) const;

	/** Returns the underlying screen reader */
	TSharedRef<FScreenReaderBase> GetScreenReader() const;

// UEngineSubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	// ~
private:
	/** 
	* The underlying screen reader. This should be a TUniquePtr
	* but we make it a TSharedPtr to allow for easy delegate unbinding without needing to manually unbind from delegates.
	*/
	TSharedPtr<FScreenReaderBase> ScreenReader;
};
