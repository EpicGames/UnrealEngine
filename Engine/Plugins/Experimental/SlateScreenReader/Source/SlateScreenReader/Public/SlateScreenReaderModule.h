// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

static_assert(WITH_ACCESSIBILITY, "Trying to use the Slate screen reader plugin with accessibility disabled. Accessibility must be enabled to use this plugin. Either enable accessibility or disable the plugin.");

class IScreenReaderBuilder;

class SLATESCREENREADER_API ISlateScreenReaderModule: public IModuleInterface
{
public:
	virtual ~ISlateScreenReaderModule() {}
	/** Get this module and load it if required. */
	static ISlateScreenReaderModule& Get()
	{
		static const FName ModuleName = "SlateScreenReader";
		return FModuleManager::Get().LoadModuleChecked<ISlateScreenReaderModule>(ModuleName);
	}
/** Returns the default IScreenReaderBuilder used to construct Slate screen readers. There will always be a valid screen reader builder for Slate screen readers.*/
	virtual TSharedRef<IScreenReaderBuilder> GetDefaultScreenReaderBuilder() const = 0;
	/** Gets the custom screen reader builder set by the user for building Slate screen readers. If the user did not set a custom screen reader builder, this returns nullptr. */
	virtual TSharedPtr<IScreenReaderBuilder> GetCustomScreenReaderBuilder() const = 0;
	/** Sets a custom screen reader builder to build Slate screen readers. */
	virtual void SetCustomScreenReaderBuilder(const TSharedRef<IScreenReaderBuilder>& InBuilder) = 0;
};

