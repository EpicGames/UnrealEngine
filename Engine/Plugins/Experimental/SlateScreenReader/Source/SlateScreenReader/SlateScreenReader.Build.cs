// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;


public class SlateScreenReader : ModuleRules
{
	public SlateScreenReader(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
			"Core",
			"CoreUObject",
			"Engine"
		});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
			"ApplicationCore",
			"SlateCore",
			"Slate",
			"InputCore",
			"TextToSpeech",
			"ScreenReader"
		});
	}
}
